// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class GPUReshapeBootstrapperTarget : TargetRules
{
	public GPUReshapeBootstrapperTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LinkType = TargetLinkType.Monolithic;
		
		bCompileICU = false;
		bBuildDeveloperTools = false;		
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;
		bCompileAgainstApplicationCore = false;
		bIsBuildingConsoleApplication = true;

		SolutionDirectory = "Programs/GPUReshape";
		LaunchModuleName = "GPUReshapeBootstrapper";
	}
}
