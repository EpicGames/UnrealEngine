// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "RequiredProgramMainCPPInclude.h"

DEFINE_LOG_CATEGORY_STATIC(LogGPUReshape, Log, All);

IMPLEMENT_APPLICATION(GPUReshapeBootstrapper, "GPUReshapeBootstrapper");

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	/**
	 * Gauntlet GPU Reshape Utility Helper
	 *
	 * Bootstrap Tree:
	 *   Gauntlet -> GPUReshapeBootstrapper -> GPUReshape -> Target
	 */
	
	FTaskTagScope Scope(ETaskTag::EGameThread);
	
	ON_SCOPE_EXIT
	{ 
		LLM(FLowLevelMemTracker::Get().UpdateStatsPerFrame());
		RequestEngineExit(TEXT("Exiting"));
		
		FEngineLoop::AppPreExit();
		FModuleManager::Get().UnloadModulesAtShutdown();
		FEngineLoop::AppExit();
	};

	if (int32 Ret = GEngineLoop.PreInit(ArgC, ArgV))
	{
		return Ret;
	}
	
#if !UE_BUILD_SHIPPING
	if (FParse::Param(FCommandLine::Get(), TEXT("WaitForDebugger")))
	{
		while (!FPlatformMisc::IsDebuggerPresent())
		{
			FPlatformProcess::Sleep(0.1f);
		}
		
		UE_DEBUG_BREAK();
	}
#endif // !UE_BUILD_SHIPPING

	FString BootstrapTarget;
	if (!FParse::Value(FCommandLine::Get(), TEXT("BootstrapTarget="), BootstrapTarget))
	{
		UE_LOGF(LogGPUReshape, Error, "Target executable path not set");
		return 1u;
	}
	
	FString GPUReshapePath;
	if (!FParse::Value(FCommandLine::Get(), TEXT("GRS.Path="), GPUReshapePath))
	{
		UE_LOGF(LogGPUReshape, Error, "GPU Reshape path not set");
		return 1u;
	}
	
	FString WorkspacePath;
	if (!FParse::Value(FCommandLine::Get(), TEXT("GRS.Workspace="), WorkspacePath))
	{
		UE_LOGF(LogGPUReshape, Error, "Workspace path not set");
		return 1u;
	}
	
	FString ReportPath;
	if (!FParse::Value(FCommandLine::Get(), TEXT("GRS.Report="), ReportPath))
	{
		UE_LOGF(LogGPUReshape, Error, "Report path not set");
		return 1u;
	}
	
	int32 Timeout = 7200;
	FParse::Value(FCommandLine::Get(), TEXT("GRS.Timeout="), Timeout);
	
	FString SymbolPath = "";
	FParse::Value(FCommandLine::Get(), TEXT("GRS.SymbolPath="), SymbolPath);

	// If relative, put the report under the saved path
	if (FPaths::IsRelative(ReportPath))
	{
		ReportPath = FPaths::Combine(FPaths::ProjectSavedDir(), ReportPath);
	}
	
	// Start in headless mode
	TStringBuilder<1024> GRSCommandLine;
	GRSCommandLine << "launch ";
	GRSCommandLine << "-report " << "\"" << ReportPath << "\" ";
	GRSCommandLine << "-workspace \"" << WorkspacePath << "\" ";
	GRSCommandLine << "-timeout " << Timeout << " ";
	GRSCommandLine << "-symbol " << SymbolPath << " ";
	GRSCommandLine << "-app " << BootstrapTarget;

	for (int32_t i = 1; i < ArgC; i++)
	{
		GRSCommandLine << " " << ArgV[i];
	}

	// Create pipes
	void *PipeRead = nullptr, *PipeWrite = nullptr;
	if (!FPlatformProcess::CreatePipe(PipeRead, PipeWrite))
	{
		UE_LOGF(LogGPUReshape, Error, "Failed to create redirect pipes");
		return 1u;
	}
	
	// Launch the editor bootstrapped through reshape
	FProcHandle Handle = FPlatformProcess::CreateProc(
		*GPUReshapePath,
		*GRSCommandLine,
		true,
		false,
		false,
		nullptr,
		0,
		nullptr,
		PipeWrite
	);
	
	if (!Handle.IsValid())
	{
		UE_LOGF(LogGPUReshape, Error, "Failed to launch bootstrapped application");
		return 1u;
	}

	// Wait for reshape to finish, redirect pipe meanwhile
	while (FPlatformProcess::IsProcRunning(Handle))
	{
		if (FString Contents = FPlatformProcess::ReadPipe(PipeRead); !Contents.IsEmpty())
		{
			UE_LOGF(LogGPUReshape, Log, "%ls", *Contents);
		}
		
		FPlatformProcess::Sleep(0.1f);
	}

	// Redirect final contents
	if (FString Contents = FPlatformProcess::ReadPipe(PipeRead); !Contents.IsEmpty())
	{
		UE_LOGF(LogGPUReshape, Log, "%ls", *Contents);
	}

	int32 ReturnCode = 1u;
	FPlatformProcess::GetProcReturnCode(Handle, &ReturnCode);
	FPlatformProcess::CloseProc(Handle);

	if (ReturnCode)
	{
		UE_LOGF(LogGPUReshape, Error, "Bootstrapped process exited with %u", ReturnCode);
	}
	
	return ReturnCode;
}
